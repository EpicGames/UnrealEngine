// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpusDecoderElectra_WinGDK : OpusDecoderElectra
	{
		public OpusDecoderElectra_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
