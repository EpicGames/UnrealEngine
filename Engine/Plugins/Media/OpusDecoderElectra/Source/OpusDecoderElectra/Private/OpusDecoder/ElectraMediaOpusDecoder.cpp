// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaOpusDecoder.h"
#include "OpusDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputAudio.h"
#include "ElectraDecodersUtils.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"
#include "MP4Utilities.h"		// for parsing the 'dOps' box

#include "opus_multistream.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				2
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				3
#define ERRCODE_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT			4

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraAudioDecoderOutputOpus_Common : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputOpus_Common()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	int32 GetNumChannels() const override
	{
		return NumChannels;
	}
	int32 GetSampleRate() const override
	{
		return SampleRate;
	}
	int32 GetNumFrames() const override
	{
		return NumFrames - PreSkip;
	}
	bool IsInterleaved() const override
	{
		return true;
	}
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < ChannelPositions.Num() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid;
	}
	ESampleFormat GetSampleFormat() const override
	{
		return ESampleFormat::Float;
	}
	int32 GetBytesPerSample() const override
	{
		return sizeof(float);
	}
	int32 GetBytesPerFrame() const override
	{
		return GetBytesPerSample() * GetNumChannels();
	}
	const void* GetData(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber + (PreSkip * NumChannels) : nullptr;
	}

public:
	TArray<EChannelPosition> ChannelPositions;
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
	int32 PreSkip = 0;
};


class FElectraOpusDecoder : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
	}

	FElectraOpusDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraOpusDecoder();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Audio;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FDOpsConfig
	{
		int32 NumberOfOutputChannels = 0;
		int32 SampleRate = 0;
		int32 PreSkip = 0;
		int32 OutputGain = 0;
		int32 ChannelMappingFamily = 0;
		int32 StreamCount = 0;
		int32 CoupledCount = 0;
		uint8 ChannelMapping[256] = {255};
		void Reset()
		{
			NumberOfOutputChannels = 0;
			SampleRate = 0;
			PreSkip = 0;
			OutputGain = 0;
			ChannelMappingFamily = 0;
			StreamCount = 0;
			CoupledCount = 0;
			FMemory::Memset(ChannelMapping, 255);
		}
		bool SameAs(const FDOpsConfig& rhs)
		{
			return NumberOfOutputChannels == rhs.NumberOfOutputChannels &&
				   SampleRate == rhs.SampleRate &&
				   PreSkip == rhs.PreSkip &&
				   OutputGain == rhs.OutputGain &&
				   ChannelMappingFamily == rhs.ChannelMappingFamily &&
				   StreamCount == rhs.StreamCount &&
				   CoupledCount == rhs.CoupledCount &&
				   FMemory::Memcmp(ChannelMapping, rhs.ChannelMapping, StreamCount) == 0;
		}
	};

	int32 OpusSamplingRate() const
	{
		if (DOpsConfig.SampleRate <= 0 || DOpsConfig.SampleRate > 24000)
		{
			return 48000;
		}
		else if (DOpsConfig.SampleRate > 16000)
		{
			return 24000;
		}
		else if (DOpsConfig.SampleRate > 12000)
		{
			return 16000;
		}
		else if (DOpsConfig.SampleRate > 8000)
		{
			return 12000;
		}
		return 8000;
	}

	bool Parse_dOps(FDOpsConfig& OutConfig, const TArray<uint8>& IndOpsBox, bool bFailOnError);

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	bool SetupChannelMap();
	bool ProcessInput(const void* InData, int64 InDataSize);

	IElectraDecoder::FError LastError;

	OpusMSDecoder* DecoderHandle = nullptr;

	TSharedPtr<FElectraAudioDecoderOutputOpus_Common, ESPMode::ThreadSafe> CurrentOutput;
	int32 RemainingPreSkip = -1;
	bool bFlushPending = false;

	// Input configuration
	FDOpsConfig DOpsConfig;
	bool bHaveParseddOps = false;

	// Output
	TArray<IElectraDecoderAudioOutput::EChannelPosition> OutputChannelMap;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraCommonAudioOpusDecoderFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FElectraCommonAudioOpusDecoderFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FElectraCommonAudioOpusDecoderFactory()
	{ }

	FElectraCommonAudioOpusDecoderFactory()
	{
		LibVersion = ANSI_TO_TCHAR(opus_get_version_string());
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FElectraOpusDecoder::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Audio || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			return 0;
		}
		if (!InCodecFormat.RFC6381.Equals(TEXT("Opus"), ESearchCase::IgnoreCase))
		{
			return 0;
		}
		if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
		{
			OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(FString(TEXT("Opus"))));
		}
		return 5;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>() ? MakeShared<FElectraOpusDecoder, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libOpus")); }
	FString GetVersion() const override
	{ return LibVersion; }
	FString GetImplementation() const override
	{ return LibName; }
	FString GetVendor() const override
	{ return(FString()); }

	FString LibName;
	FString LibVersion;
	static TSharedPtr<FElectraCommonAudioOpusDecoderFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FElectraCommonAudioOpusDecoderFactory, ESPMode::ThreadSafe> FElectraCommonAudioOpusDecoderFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaOpusDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FElectraCommonAudioOpusDecoderFactory::Self.IsValid());
	FElectraCommonAudioOpusDecoderFactory::Self = MakeShared<FElectraCommonAudioOpusDecoderFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioOpusDecoderFactory::Self.Get());
}

void FElectraMediaOpusDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioOpusDecoderFactory::Self.Get());
	FElectraCommonAudioOpusDecoderFactory::Self.Reset();
}

TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FElectraMediaOpusDecoder::CreateFactory()
{
	return MakeShared<FElectraCommonAudioOpusDecoderFactory, ESPMode::ThreadSafe>();;
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FElectraOpusDecoder::FElectraOpusDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.DCR.Num())
	{
		bHaveParseddOps = Parse_dOps(DOpsConfig, InCodecFormat.DCR, false);
	}
	if (!bHaveParseddOps && InCodecFormat.ExtraBoxes.Contains(MP4Utilities::MakeBoxAtom('d','O','p','s')))
	{
		bHaveParseddOps = Parse_dOps(DOpsConfig, InCodecFormat.ExtraBoxes[MP4Utilities::MakeBoxAtom('d','O','p','s')], false);
	}
}

FElectraOpusDecoder::~FElectraOpusDecoder()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

bool FElectraOpusDecoder::Parse_dOps(FDOpsConfig& OutConfig, const TArray<uint8>& IndOpsBox, bool bFailOnError)
{
	if (IndOpsBox.Num() == 0)
	{
		return !bFailOnError ? false : PostError(0, TEXT("There is no 'dOps' box to get Opus information from"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}
	else if (IndOpsBox.Num() < 11)
	{
		return !bFailOnError ? false : PostError(0, TEXT("Incomplete 'dOps' box"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}

	MP4Utilities::FMP4AtomReader rd(IndOpsBox);
	uint32 Value32;
	uint16 Value16;
	uint8 Value8;
	rd.Read(Value8);		// Version
	if (Value8 != 0)
	{
		return !bFailOnError ? false : PostError(0, TEXT("Unsupported 'dOps' box version"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}
	rd.Read(Value8);		// OutputChannelCount
	rd.Read(Value16);		// PreSkip
	rd.Read(Value32);		// InputSampleRate
	OutConfig.NumberOfOutputChannels = Value8;
	OutConfig.PreSkip = Value16;
	OutConfig.SampleRate = Value32;
	rd.Read(Value16);		// OutputGain
	OutConfig.OutputGain = Value16;
	rd.Read(Value8);		// ChannelMappingFamily
	OutConfig.ChannelMappingFamily = Value8;
	if (OutConfig.ChannelMappingFamily)
	{
		/*
			Channel mapping family:
				0 : mono, L/R stereo
				1 : 1-8 channel surround
				2 : Ambisonics with individual channels
				3 : Ambisonics with demixing matrix
				255 : discrete channels
		*/
		if (OutConfig.ChannelMappingFamily == 2 || OutConfig.ChannelMappingFamily == 3)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Ambisonics is not supported"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		else if (OutConfig.ChannelMappingFamily > 3 && OutConfig.ChannelMappingFamily < 255)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Unsupported channel mapping family"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		if (rd.GetNumBytesRemaining() < 2+OutConfig.NumberOfOutputChannels)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Incomplete 'dOps' box"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		rd.Read(Value8);	// StreamCount
		OutConfig.StreamCount = Value8;
		rd.Read(Value8);	// CoupledCount
		OutConfig.CoupledCount = Value8;
		// Pre-set channel mapping array to all disabled.
		FMemory::Memset(OutConfig.ChannelMapping, 255);
		/*
			The channel mapping array determines the order in which the input stream is mapped to the
			output channels. The default order produces output according to the Vorbis order
			 (see: https://www.xiph.org/vorbis/doc/Vorbis_I_spec.html#x1-810004.3.9 )
			for 1-8 channels. More than 8 channels have essentially no defined order.

			We could change the order here to produce output in any which order we like, including
			disabling channels we are not interested in, but we do not do this here as we have our
			own channel mapper that we use for this.

			Internally Opus first decodes coupled streams, followed by uncoupled ones.
			Coupled streams are stereo and thus have 2 channels, while uncoupled streams are single channel only.
			The index values in this table are defined like this:
				For j=0; j<OutputChannelCount:
					Index = ChannelMapping[j];
					If Index == 255:
						OutputChannel[j] is made silent
					Elseif Index < 2*CoupledCount:
						OutputChannel[j] is taken from coupled stream [Index/2][Index&1]
					Else:
						OutputChannel[j] is taken from uncoupled stream [Index - CoupledCount]

			You do not necessarily know which stream contains which original input channel, which is why
			the default mapping table exists and indicates the abovementioned Vorbis mapping.
		*/
		for(int32 i=0; i<OutConfig.NumberOfOutputChannels; ++i)
		{
			rd.Read(OutConfig.ChannelMapping[i]);
		}
	}
	else
	{
		OutConfig.StreamCount = 1;
		OutConfig.CoupledCount = OutConfig.NumberOfOutputChannels > 1 ? 1 : 0;
		OutConfig.ChannelMapping[0] = 0;
		OutConfig.ChannelMapping[1] = 1;
	}
	return true;
}



bool FElectraOpusDecoder::InternalDecoderCreate()
{
	if (!DecoderHandle)
	{
		int32 DecoderAllocSize = (int32) opus_multistream_decoder_get_size(DOpsConfig.StreamCount, DOpsConfig.CoupledCount);
		if (DecoderAllocSize < 0)
		{
			return PostError(DecoderAllocSize, TEXT("opus_multistream_decoder_get_size() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		}
		DecoderHandle = (OpusMSDecoder*) FMemory::Malloc(DecoderAllocSize);
		int32 Result = opus_multistream_decoder_init(DecoderHandle, OpusSamplingRate(), DOpsConfig.NumberOfOutputChannels, DOpsConfig.StreamCount, DOpsConfig.CoupledCount, DOpsConfig.ChannelMapping);
		if (Result != OPUS_OK)
		{
			return PostError(Result, TEXT("opus_multistream_decoder_init() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		}
		RemainingPreSkip = -1;
	}
	return true;
}

void FElectraOpusDecoder::InternalDecoderDestroy()
{
	if (DecoderHandle)
	{
		FMemory::Free(DecoderHandle);
		DecoderHandle = nullptr;
	}
}


void FElectraOpusDecoder::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraOpusDecoder::GetError() const
{
	return LastError;
}

bool FElectraOpusDecoder::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FElectraOpusDecoder::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FElectraOpusDecoder::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// No configuration parsed yet, so this is deemed compatible.
	if (!bHaveParseddOps)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	TArray<uint8> DCR = ElectraDecodersUtil::GetVariantValueUInt8Array(CSDAndAdditionalOptions, TEXT("dcr"));
	TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(CSDAndAdditionalOptions, TEXT("csd"));
	FDOpsConfig cfg;
	if (!Parse_dOps(cfg, DCR.Num() ? DCR : CSD, false))
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	if (cfg.SameAs(DOpsConfig))
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraOpusDecoder::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	RemainingPreSkip = -1;

	bHaveParseddOps = false;
	DOpsConfig.Reset();
	OutputChannelMap.Empty();
	InternalDecoderDestroy();
	return true;
}

bool FElectraOpusDecoder::SetupChannelMap()
{
	if (OutputChannelMap.Num())
	{
		return true;
	}
	// Pre-init with all channels disabled.
	OutputChannelMap.Empty();
	OutputChannelMap.Init(IElectraDecoderAudioOutput::EChannelPosition::Disabled, DOpsConfig.NumberOfOutputChannels);

	if (DOpsConfig.ChannelMappingFamily == 0)
	{
		if (DOpsConfig.NumberOfOutputChannels != 1 && DOpsConfig.NumberOfOutputChannels != 2)
		{
			return PostError(0, FString::Printf(TEXT("Unsupported number of channels (%d) for mapping family 0"), DOpsConfig.NumberOfOutputChannels), ERRCODE_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT);
		}
		if (DOpsConfig.NumberOfOutputChannels == 1)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::C;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 2)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		}
	}
	else if (DOpsConfig.ChannelMappingFamily == 1)
	{
		// Channel mapping family 1 provides a default channel mapping table in Vorbis channel order.
		// We use that order to remap the channels to our positions.
		if (DOpsConfig.NumberOfOutputChannels == 1)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::C;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 2)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 3)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::C;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::R;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 4)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
			OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 5)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::C;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::R;
			OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
			OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 6)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::C;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::R;
			OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
			OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
			OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 7)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::C;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::R;
			OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
			OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
			OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::Cs;
			OutputChannelMap[6] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		}
		else if (DOpsConfig.NumberOfOutputChannels == 8)
		{
			OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
			OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::C;
			OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::R;
			OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
			OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
			OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::Lsr;
			OutputChannelMap[6] = IElectraDecoderAudioOutput::EChannelPosition::Rsr;
			OutputChannelMap[7] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		}
		else
		{
			return PostError(0, FString::Printf(TEXT("Unsupported number of channels (%d) for mapping family 1"), DOpsConfig.NumberOfOutputChannels), ERRCODE_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT);
		}
	}
	else if (DOpsConfig.ChannelMappingFamily == 255)
	{
		// Unspecified order
		for(int32 i=0; i<DOpsConfig.NumberOfOutputChannels; ++i)
		{
			OutputChannelMap[i] = static_cast<IElectraDecoderAudioOutput::EChannelPosition>(static_cast<int32>(IElectraDecoderAudioOutput::EChannelPosition::Unspec0) + i);
		}
	}
	else
	{
		return PostError(0, FString::Printf(TEXT("Unsupported channel mapping family (%d)"), DOpsConfig.ChannelMappingFamily), ERRCODE_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT);
	}
	return true;
}


bool FElectraOpusDecoder::ProcessInput(const void* InData, int64 InDataSize)
{
	if (!DecoderHandle)
	{
		return false;
	}

	// Set the pre-skip if necessary (first packet)
	if (RemainingPreSkip < 0)
	{
		RemainingPreSkip = DOpsConfig.PreSkip;
	}

	int32 NumExpectedDecodedSamples = opus_packet_get_nb_samples(reinterpret_cast<const unsigned char*>(InData), (int32)InDataSize, OpusSamplingRate());
	if (NumExpectedDecodedSamples < 0)
	{
		return PostError(NumExpectedDecodedSamples, TEXT("opus_packet_get_nb_samples() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
	}

	int32 AllocSize = sizeof(float) * DOpsConfig.NumberOfOutputChannels * NumExpectedDecodedSamples;
	CurrentOutput->Buffer = (float*)FMemory::Malloc(AllocSize);
	CurrentOutput->ChannelPositions = OutputChannelMap;
	CurrentOutput->NumChannels = DOpsConfig.NumberOfOutputChannels;
	CurrentOutput->SampleRate = OpusSamplingRate();

	int32 Result = opus_multistream_decode_float(DecoderHandle, reinterpret_cast<const unsigned char*>(InData), (int32)InDataSize, CurrentOutput->Buffer, NumExpectedDecodedSamples, 0);
	if (Result < 0)
	{
		return PostError(Result, TEXT("opus_multistream_decode_float() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
	}
	CurrentOutput->NumFrames = Result;

#if 0
	// Do not apply the pre-skip here. The upper layer decoder is responsible for adjusting the
	// decoded samples based on the PTS value, which will be negative at the start to account
	// for the duration of the pre-skip.

	// Apply pre skip
	if (RemainingPreSkip > 0)
	{
		// Entire output to be discarded?
		if (RemainingPreSkip >= CurrentOutput->NumFrames)
		{
			CurrentOutput.Reset();
		}
		else
		{
			CurrentOutput->PreSkip = RemainingPreSkip;
			// We need to advance the PTS of this output.
			FTimespan SkipTime(ETimespan::TicksPerSecond * RemainingPreSkip / CurrentOutput->SampleRate);
			CurrentOutput->PTS += SkipTime;
		}
		// Done?
		if ((RemainingPreSkip -= CurrentOutput->NumFrames) < 0)
		{
			RemainingPreSkip = 0;
		}
	}
#endif
	return true;
}

IElectraDecoder::EDecoderError FElectraOpusDecoder::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode data.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Parse the codec specific information
		if (!bHaveParseddOps)
		{
			if (InAdditionalOptions.Contains(TEXT("dcr")))
			{
				bHaveParseddOps = Parse_dOps(DOpsConfig, ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("dcr")), true);
			}
			if (!bHaveParseddOps && InAdditionalOptions.Contains(TEXT("dOps")))
			{
				bHaveParseddOps = Parse_dOps(DOpsConfig, ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("dOps")), true);
			}
			if (!bHaveParseddOps)
			{
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		// Set up the channel map accordingly.
		if (!SetupChannelMap())
		{
			// Error was already posted.
			return IElectraDecoder::EDecoderError::Error;
		}
		// Create decoder if necessary.
		if (!DecoderHandle && !InternalDecoderCreate())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		// Prepare the output
		if (!CurrentOutput.IsValid())
		{
			CurrentOutput = MakeShared<FElectraAudioDecoderOutputOpus_Common>();
			CurrentOutput->PTS = InInputAccessUnit.PTS;
			CurrentOutput->UserValue = InInputAccessUnit.UserValue;
		}
		// Decode
		if (!ProcessInput(InInputAccessUnit.Data, InInputAccessUnit.DataSize))
		{
			CurrentOutput.Reset();
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraOpusDecoder::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraOpusDecoder::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraOpusDecoder::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		RemainingPreSkip = -1;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraOpusDecoder::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
