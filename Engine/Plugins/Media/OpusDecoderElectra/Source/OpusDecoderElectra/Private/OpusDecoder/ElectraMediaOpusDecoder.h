// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IElectraCodecFactory;

class FElectraMediaOpusDecoder
{
public:
	static void Startup();
	static void Shutdown();
	static TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> CreateFactory();
};
