// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpusDecoderElectraModule.h"
#include "OpusDecoder/ElectraMediaOpusDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "OpusDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogOpusElectraDecoder);

class FOpusElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaOpusDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaOpusDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FOpusElectraDecoderModule, OpusDecoderElectra);

#undef LOCTEXT_NAMESPACE
